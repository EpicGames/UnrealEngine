// Copyright Epic Games, Inc. All Rights Reserved.

using System.Globalization;
using System.IO;
using UnrealBuildBase;
using UnrealBuildTool;

public class BreakpadSymbolEncoder : ModuleRules
{
	public BreakpadSymbolEncoder(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");

		bAddDefaultIncludePaths = false;
	}
}
