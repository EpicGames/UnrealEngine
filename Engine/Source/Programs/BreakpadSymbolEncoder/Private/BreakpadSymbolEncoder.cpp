// Copyright Epic Games, Inc. All Rights Reserved.

#include "BreakpadSymbolEncoder.h"

#include <algorithm>
#include <iostream>
#include <fstream>
#include <limits>
#include <string>
#include <string.h>
#include <unordered_map>
#include <vector>


#define VERBOSE_DEBUG 0

namespace
{
std::string ReadInFile(const std::string& Path)
{
	std::ifstream InFile(Path.c_str(), std::ios::in | std::ios::binary);

	if (InFile)
	{
		std::string Out;
		InFile.seekg(0, std::ios::end);

		Out.resize(InFile.tellg());
		InFile.seekg(0, std::ios::beg);
		InFile.read(&Out[0], Out.size());
		InFile.close();

		return Out;
	}

	return {};
}

std::vector<std::string> SplitFilePerLine(std::string&& RawBytes)
{
	std::vector<std::string> Out;

	size_t Current = 0;
	while (Current != std::string::npos && Current < RawBytes.size())
	{
		size_t NewLineEnd = RawBytes.find("\n", Current);
		size_t NewLineSize = 1;

		// Check if we are CRLF \r\n
		if (NewLineEnd > 0 && NewLineEnd != std::string::npos)
		{
			if (RawBytes[NewLineEnd - 1] == '\r')
			{
				NewLineEnd--;
				NewLineSize = 2;
			}

			Out.push_back(RawBytes.substr(Current, NewLineEnd - Current));
			Current = NewLineEnd + NewLineSize;
		}
		else
		{
			Out.push_back(RawBytes.substr(Current));
			break;
		}
	}

	return Out;
}

std::vector<std::string> SplitLineIntoNEntries(const std::string& Line, size_t n)
{
	std::vector<std::string> Out;

	size_t Current = 0;
	for (size_t i = 0; i < n; i++)
	{
		size_t End = Line.find(" ", Current);
		Out.push_back(Line.substr(Current, End - Current));
		Current = End + 1;
	}

	Out.push_back(Line.substr(Current));

	if (Out.size() != n + 1)
	{
		std::cerr << "Failed to split the string by an expected amount\n";
		return {};
	}

	return Out;
}

bool BeginsWith(const std::string& String, const std::string& With)
{
	return String.compare(0, With.size(), With) == 0;
}

// For the symbol file only expect lower case letters for hex
constexpr bool IsHex(char c)
{
	return (c >= '0' && c <= '9') || (c >= 'a' && c <= 'z');
}

bool BeginsWithHex(const std::string& String)
{
	for (size_t i = 0; i < String.size(); i++)
	{
		if (i > 0 && String[i] == ' ')
		{
			break;
		}
		else if (!IsHex(String[i]))
		{
			return false;
		}
	}

	return true;
}

/* Used to keep track of the actual sizes of FUNCs so we can generate
 * Records to fill in the gap between FUNCs as this would be a major issue.
 * PUBLIC symbols are assumed to have a size extending to the next symbol so
 * these dont require any dummy Records to fill in the gaps
 */
struct SymbolChunk
{
	uint64_t Address;
	uint32_t Size;
	bool bPublic;

	friend bool operator<(const SymbolChunk& A, const SymbolChunk& B)
	{
		return A.Address < B.Address;
	}
};

/* Keeps track of the Filename as well as a relative offset in bytes
 * from the start of a contigous chunk of memory to its current location
 */
struct FileWithOffset
{
	std::string Name;
	uint32_t RelativeOffset;
};

/* Keeps track of the Symbol name as well as a relative offset in bytes
 * from the start of a contigous chunk of memory to its current location
 */
struct SymbolWithOffset
{
	std::string Name;
	uint32_t RelativeOffset;
};

// PUBLIC address parameter_size name
struct PublicRecord
{
	uint64_t Address;
	std::string ParameterSize;
	std::string Name;
};

// https://github.com/google/breakpad/blob/master/docs/symbol_files.md
//
// Prefix	: Info								   : Number of spaces
// ------------------------------------------------------------------
// MODULE	: operatingsystem architecture id name : 4
// FILE		: number name						   : 2
// FUNC m	: address size parameter_size name	   : 5
// FUNC		: address size parameter_size name	   : 4
// address	: size line filenum					   : 3
// PUBLIC m : address parameter_size name		   : 4
// PUBLIC	: address parameter_size name		   : 3
// STACK	:									   : 0 // Ignore
// INFO		:									   : 0 // Ignore

/* We use these to split up an expected LINE into its assumed line. Gets around needing a
 * lexer/parser where we can just assume the number of spaces and from there what is what
 */
const size_t ExpectedFileSpaces   = 2;
const size_t ExpectedFuncSpaces   = 4;
const size_t ExpectedLineSpaces   = 3;
const size_t ExpectedPublicSpaces = 3;

bool ParseSymbolFile(const std::string& SymbolFile, std::vector<Record>& out_Records, std::vector<FileWithOffset>& out_FileRecords, std::vector<SymbolWithOffset>& out_SymbolNames)
{
	std::unordered_map<uint64_t, uint64_t> FuncRecords;
	std::vector<PublicRecord> PublicRecords;
	std::vector<SymbolChunk> SymbolChunks;

	uint32_t RelativeFileOffset		  = 0;
	uint32_t RelativeSymbolNameOffset = 0;

	int LineCount	= 0;
	int IgnoreCount = 0;
	int ActualLineCount = 0;

	bool bFirstLineRecordFromFunc = false;

	std::vector<std::string> SplitLines = SplitFilePerLine(ReadInFile(SymbolFile));

	if (SplitLines.empty())
	{
		std::cerr << "Failed to read file: '" << SymbolFile << "'" << '\n';
		return false;
	}

	for (auto const& Line : SplitLines)
	{
		// address size line filenum
		if (BeginsWithHex(Line))
		{
			Record Out;

			char const* RawLine = Line.c_str();
			char* End;

			Out.Address = std::strtoull(RawLine, &End, 16);
			// Need to skip the size entry
			End = strchr(End + 1, ' ');
			Out.LineNumber = std::strtoul(End, &End, 10);
			// Store the actual index into the contiguous memory, which we will convert into a RelativeOffset at the end when we have the totals
			Out.FileRelativeOffset = std::strtoul(End, nullptr, 10);

			/* An example of what a FUNC + LINE records would look like:
			 * FUNC
			 * LINE RECORD LineNumber FileNumber
			 * LINE RECORD LineNumber FileNumber
			 * ....
			 * LINE RECORD LineNumber FileNumber
			 *
			 * Compress simply ignores a LINE RECORD *if* the previous LINE RECORD has the same LineNumber and FileNumber
			 * This will give us a larger Chunk size for this entry but for our use case its not required.
			 */

			if (bFirstLineRecordFromFunc)
			{
				// Same as the FileAbsoluteOffset but for the Symbol name. Need to covert at the end when all the memory we are going to write is in the structure
				Out.SymbolRelativeOffset = static_cast<uint32_t>(out_SymbolNames.size() - 1);
				out_Records.emplace_back(Out);
				bFirstLineRecordFromFunc = false;
				ActualLineCount++; // TODO Remove just for numbers
			}
			else
			{
				Record LastRecord = out_Records.back();

				// We at times have a line number of zero from dump_syms, this can mess up compressing chunks of the same PC range
				// just to save on size lets just assume if its a zero, use the previous line number. While not correct, not any less
				// incorrect then 0, and helps reduce the sym file size by a good amount
				if (Out.LineNumber == 0)
				{
					Out.LineNumber = LastRecord.LineNumber;
				}

				if (LastRecord.LineNumber != Out.LineNumber || LastRecord.FileRelativeOffset != Out.FileRelativeOffset)
				{
					// Same as the FileAbsoluteOffset but for the Symbol name. Need to covert at the end when all the memory we are going to write is in the structure
					Out.SymbolRelativeOffset = static_cast<uint32_t>(out_SymbolNames.size() - 1);
					out_Records.emplace_back(Out);
					ActualLineCount++; // TODO Remove just for numbers
				}
			}

			LineCount++;
		}
		// FUNC address size parameter_size name
		else if (BeginsWith(Line, "FUNC"))
		{
			size_t FuncSpaces = ExpectedFuncSpaces;
			size_t FirstValue = 1;
			if (BeginsWith(Line, "FUNC m"))
			{
				FuncSpaces++;
				FirstValue++;
			}

			std::vector<std::string> FuncSplit = SplitLineIntoNEntries(Line, FuncSpaces);
			if (!FuncSplit.empty())
			{
				uint64_t Address = std::strtoull(FuncSplit[FirstValue].c_str(), 0, 16);
				uint32_t Size = std::strtoul(FuncSplit[FirstValue + 1].c_str(), 0, 16);

				FuncRecords[Address] = Address;
				out_SymbolNames.push_back({FuncSplit[FirstValue + 3] + "\n", RelativeSymbolNameOffset});
				RelativeSymbolNameOffset += static_cast<uint32_t>(out_SymbolNames.back().Name.size()) * sizeof(char);

				SymbolChunks.push_back({Address, Size, false});
				bFirstLineRecordFromFunc = true;
			}
			else
			{
				std::cerr << "ERROR: Failed to split a FUNC line:\n  " << Line << '\n';
			}
		}
		// PUBLIC address parameter_size name
		else if (BeginsWith(Line, "PUBLIC"))
		{
			size_t PublicSpaces = ExpectedPublicSpaces;
			size_t FirstValue = 1;

			if (BeginsWith(Line, "PUBLIC m"))
			{
				PublicSpaces++;
				FirstValue++;
			}

			std::vector<std::string> PublicSplit = SplitLineIntoNEntries(Line, PublicSpaces);
			if (!PublicSplit.empty())
			{
				PublicRecords.push_back({
					std::strtoull(PublicSplit[FirstValue].c_str(), 0, 16),
					PublicSplit[FirstValue + 1],
					PublicSplit[FirstValue + 2]
				});
			}
			else
			{
				std::cerr << "ERROR: Failed to split a PUBLIC line:\n  " << Line << '\n';
			}
		}
		// FILE number name
		else if (BeginsWith(Line, "FILE"))
		{
			std::vector<std::string> FileSplit = SplitLineIntoNEntries(Line, ExpectedFileSpaces);
			if (!FileSplit.empty())
			{
				// Add a newline as we'll need to use that when reading later
				std::string Filename = FileSplit[2] + "\n";
				// Maintain one style of pathing
				std::replace(std::begin(Filename), std::end(Filename), '\\', '/');
				out_FileRecords.push_back({Filename, RelativeFileOffset});
				RelativeFileOffset += static_cast<uint32_t>(out_FileRecords.back().Name.size()) * sizeof(char);
			}
			else
			{
				std::cerr << "ERROR: Failed to split a FILE line:\n  " << Line << '\n';
			}
		}
		else if (BeginsWith(Line, "STACK")  ||
				 BeginsWith(Line, "INFO")   ||
				 BeginsWith(Line, "MODULE") ||
				 BeginsWith(Line, "INLINE"))
		{
			// Ignore
			IgnoreCount++;
		}
		else
		{
			std::cerr << "ERROR: Unepxected line: " <<	Line << '\n';
			return false;
		}
	}

	// Only add Records for PUBLIC symbols that are not already captured by a FUNC entry
	int TotalPublicKept = 0;
	for (auto const& PRecord : PublicRecords)
	{
		if (FuncRecords.find(PRecord.Address) == FuncRecords.end())
		{
			Record Out;
			Out.Address            = PRecord.Address;
			Out.FileRelativeOffset = static_cast<uint32_t>(-1);
			Out.LineNumber         = static_cast<uint32_t>(-1);

			out_SymbolNames.push_back({PRecord.Name + "\n", RelativeSymbolNameOffset});
			RelativeSymbolNameOffset += static_cast<uint32_t>(out_SymbolNames.back().Name.size()) * sizeof(char);

			// We just pushed a new symbol on the list, use that as the index when we look up offsets later
			Out.SymbolRelativeOffset = static_cast<uint32_t>(out_SymbolNames.size() - 1);
			out_Records.emplace_back(Out);

			// Add all the PUBLIC symbols we need to account for so we dont add dummy Records in their locations
			SymbolChunks.push_back({Out.Address, 0, true});

			TotalPublicKept++;
		}
	}

	// We have put all the FUNC and PUBLIC (non duplicates), need to sort them before generating the dummy entries
	std::sort(SymbolChunks.begin(), SymbolChunks.end());

	// Dummy symbol name
	out_SymbolNames.push_back({"?????????????\n", RelativeSymbolNameOffset});
	RelativeSymbolNameOffset += static_cast<uint32_t>(out_SymbolNames.back().Name.size()) * sizeof(char);

	int ChunksAdded = 0;
	for (size_t i = 0; i < SymbolChunks.size() - 1; i++)
	{
		// We assume all public symbols extend to the next symbol
		if (!SymbolChunks[i].bPublic)
		{
			uint64_t Address	 = SymbolChunks[i].Address;
			uint64_t NextAddress = SymbolChunks[i + 1].Address;
			uint32_t Size		 = SymbolChunks[i].Size;

			if (Address + Size != NextAddress)
			{
				// Add a dummy symbol that fills in the Hole between symbols so we can assume NextAddress - Address == Size
				out_Records.push_back({
					Address + Size,
					static_cast<uint32_t>(-1),
					static_cast<uint32_t>(-1),
					static_cast<uint32_t>(out_SymbolNames.size() - 1)
				});

				ChunksAdded++;
			}
		}
	}

	// Add a final dummy record for the last entry. This way you can get the size of the last entry (which is just assumed to be 4 bytes as it'll
	// be a public function with no defined size).
	out_Records.push_back({
		out_Records.back().Address + 0x4,
		static_cast<uint32_t>(-1),
		static_cast<uint32_t>(-1),
		static_cast<uint32_t>(out_SymbolNames.size() - 1)
	});

#if VERBOSE_DEBUG
	std::cout << "TotalLines: " << LineCount << " Actual Lines Added: " << ActualLineCount << " Percent compressed: " << 100 - (ActualLineCount / (float)LineCount * 100) << "%" << '\n';
	std::cout << "TotalPublic: " << PublicRecords.size() << " Actual Public Added: " << TotalPublicKept << " Percent removed: " << 100 - (TotalPublicKept / (float)PublicRecords.size() * 100) << "%" << '\n';

	std::cout << std::dec  << "File:   " << out_FileRecords.size() << "\t" << (out_FileRecords.size() / (float)SplitLines.size()) * 100 << '\n'
						   << "Func:   " << FuncRecords.size()	   << "\t" << (FuncRecords.size() / (float)SplitLines.size()) * 100 << '\n'
						   << "Public: " << PublicRecords.size()   << "\t" << (PublicRecords.size() / (float)SplitLines.size()) * 100 << '\n'
						   << "Ignore: " << IgnoreCount			   << "\t" << (IgnoreCount / (float)SplitLines.size()) * 100 << '\n'
						   << "Line:   " << LineCount			   << "\t" << (LineCount / (float)SplitLines.size()) * 100 << '\n'
						   << "Total:  " << SplitLines.size()	   << '\n';

	std::cout << "Total Record:  " << out_Records.size() << "\n"
		      << "  TotalLines:  " << ActualLineCount << "\n"
		      << "  TotalPublic: " << TotalPublicKept << "\n"
		      << "  EmptyChunks: " << ChunksAdded << "\n";
#endif


	return true;
}

void EncodeSymbolFile(const std::string& SymbolFile, const std::string& OutputFile)
{
	std::vector<Record> Records;
	std::vector<FileWithOffset> FileRecords;
	std::vector<SymbolWithOffset> SymbolNames;
	std::string Filename;

	if (!ParseSymbolFile(SymbolFile, Records, FileRecords, SymbolNames))
	{
		std::cerr << "Failed to parse '" << SymbolFile << "'" << '\n';
		return;
	}

	if (SymbolFile.empty() || OutputFile.empty())
	{
		std::cerr << "ERROR: Symbol file or Output file is empty '" << SymbolFile << "' '" << OutputFile << "'" << '\n';
		return;
	}

	std::sort(Records.begin(), Records.end());

	uint64_t RecordsSize = Records.size() * sizeof(Record);

	uint64_t FilesBytesSize = 0;
	for (size_t i = 0; i < FileRecords.size(); i++)
	{
		FilesBytesSize += FileRecords[i].Name.size() * sizeof(char);
	}

	uint64_t SymbolBytesSize = 0;
	for (size_t i = 0; i < SymbolNames.size(); i++)
	{
		SymbolBytesSize += SymbolNames[i].Name.size() * sizeof(char);
	}

	if (FilesBytesSize + SymbolBytesSize > std::numeric_limits<uint32_t>::max())
	{
		std::cerr << "ERROR: String section larger then the uint32_t::max() cannot encode the offsets" << '\n';
		return;
	}

	if (Records.size() > std::numeric_limits<uint32_t>::max())
	{
		std::cerr << "ERROR: Record count greater then uint32_t::max() cannot encode the record count" << '\n';
		return;
	}

	// Replace all the stored index with relative offsets from the start of the strings section in the output file
	for (auto& R : Records)
	{
		if (R.FileRelativeOffset < FileRecords.size())
		{
			R.FileRelativeOffset = FileRecords[R.FileRelativeOffset].RelativeOffset;
		}
		else if (R.FileRelativeOffset != (uint32_t)-1)
		{
			std::cerr << "Error FileRelativeOffset larger then expected range, got: " << R.FileRelativeOffset << " Expect less then: " << FileRecords.size() << '\n';
		}

		if (R.SymbolRelativeOffset != static_cast<uint32_t>(-1))
		{
			R.SymbolRelativeOffset = SymbolNames[R.SymbolRelativeOffset].RelativeOffset + static_cast<uint32_t>(FilesBytesSize);
		}
	}

	// If we require larger then 4GB files... we'll need to reconsider this
	RecordsHeader Header{static_cast<uint32_t>(Records.size())};

	std::ofstream os(OutputFile, std::ios::binary);
	if (os.is_open())
	{
		os.write((char*)&Header, sizeof(RecordsHeader));
		os.write((char*)Records.data(), RecordsSize);

		for (size_t i = 0; i < FileRecords.size(); i++)
		{
			os.write((char*)&FileRecords[i].Name[0], FileRecords[i].Name.size() * sizeof(char));
		}

		for (size_t i = 0; i < SymbolNames.size(); i++)
		{
			os.write((char*)&SymbolNames[i].Name[0], SymbolNames[i].Name.size() * sizeof(char));
		}

		os.close();

#if VERBOSE_DEBUG
		std::cout << "	OutputFile: " << OutputFile << '\n';
		std::cout << "	RecordsSize : 0x" << std::hex << RecordsSize << '\n';
		std::cout << "	RecordOut Offset: 0x" << std::hex << RecordsSize + sizeof(RecordsHeader) << '\n';
		std::cout << "	Record + Files Offset: 0x" << RecordsSize  + FilesBytesSize << std::dec << '\n';
#endif
	}
	else
	{
		std::cerr << "ERROR: Failed to open file for writing: " << OutputFile << '\n';
	}
}
}

int main(int argc, char* argv[])
{
	if (argc > 2)
	{
		std::string SymbolFile(argv[1]);
		std::string OutputFile(argv[2]);

#if VERBOSE_DEBUG
		std::cout << "Attempting to read Symbol file: '" << SymbolFile << "'" << '\n';
#endif

		EncodeSymbolFile(SymbolFile, OutputFile);
	}
	else
	{
		std::cerr << "Usage: " << argv[0] << " <path/to/symbol/file> <path/to/output/file>" << '\n';
	}
}
