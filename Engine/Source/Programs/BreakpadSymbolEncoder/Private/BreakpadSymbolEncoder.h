// Copyright Epic Games, Inc. All Rights Reserved.

/* The output files are packed in this ordered:
 *
 * RecordsHeader
 * Records
 * File strings
 * Symbol strings 
 *
 * Start of strings section:
 *	 RecordsHeader::RecordCount * sizeof(Record) + sizeof(RecordHeader);
 *
 */

#include <stdint.h>

#pragma pack(push, 1)
struct RecordsHeader
{
	/* The number of Records encoded in the output file */
	uint32_t RecordCount;
};

struct Record
{
	/* The Address of the Symbol */
	uint64_t Address;

	/* The Line number of the symbol in the File */
	uint32_t LineNumber;

	/* The Relative Offset in bytes from the start of the strings to the start of the File name in the output file */
	uint32_t FileRelativeOffset;

	/* The Relative Offset in bytes from the start of the strings to the start of the Symbol name in the output file */
	uint32_t SymbolRelativeOffset;

	friend bool operator<(const Record& A, const Record& B)
	{
		return A.Address < B.Address;
	}
};
#pragma pack(pop)
